// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioCaptureCore : ModuleRules
	{
		public AudioCaptureCore(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "SignalProcessing"
                }
            );

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine"
				}
			);
		}
	}
}
