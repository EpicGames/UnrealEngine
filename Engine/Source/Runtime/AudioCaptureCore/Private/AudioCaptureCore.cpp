// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioCaptureCore.h"
#include "AudioCaptureInternal.h"
#include "CoreMinimal.h"



namespace Audio
{
	static const unsigned int MaxBufferSize = 2 * 2 * 48000;

	FAudioCapture::FAudioCapture()
	{
		Impl = CreateImpl();
	}

	FAudioCapture::~FAudioCapture()
	{
	}

	bool FAudioCapture::RegisterUser(const TCHAR* UserId)
	{
		if (Impl.IsValid())
		{
			return Impl->RegisterUser(UserId);
		}

		return false;
	}

	bool FAudioCapture::UnregisterUser(const TCHAR* UserId)
	{
		if (Impl.IsValid())
		{
			return Impl->UnregisterUser(UserId);
		}

		return false;
	}

	int32 FAudioCapture::GetCaptureDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices)
	{
		if (Impl.IsValid())
		{
			return Impl->GetInputDevicesAvailable(OutDevices);
		}
		else
		{
			return false;
		}
	}

	bool FAudioCapture::GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex)
	{
		if (Impl.IsValid())
		{
			return Impl->GetCaptureDeviceInfo(OutInfo, DeviceIndex);
		}

		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool FAudioCapture::OpenCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnCaptureFunction OnCapture, uint32 NumFramesDesired)
	{
		if (Impl.IsValid())
		{
			return Impl->OpenCaptureStream(InParams, MoveTemp(OnCapture), NumFramesDesired);
		}

		return false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FAudioCapture::OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction OnCapture, uint32 NumFramesDesired)
	{
		if (Impl.IsValid())
		{
			return Impl->OpenAudioCaptureStream(InParams, MoveTemp(OnCapture), NumFramesDesired);
		}

		return false;
	}

	bool FAudioCapture::CloseStream()
	{
		if (Impl.IsValid())
		{
			return Impl->CloseStream();
		}
		return false;
	}

	bool FAudioCapture::StartStream()
	{
		if (Impl.IsValid())
		{
			return Impl->StartStream();
		}
		return false;
	}

	bool FAudioCapture::StopStream()
	{
		if (Impl.IsValid())
		{
			return Impl->StopStream();
		}
		return false;
	}

	bool FAudioCapture::AbortStream()
	{
		if (Impl.IsValid())
		{
			return Impl->AbortStream();
		}
		return false;
	}

	bool FAudioCapture::GetStreamTime(double& OutStreamTime) const
	{
		if (Impl.IsValid())
		{
			return Impl->GetStreamTime(OutStreamTime);
		}
		return false;
	}

	int32 FAudioCapture::GetSampleRate() const
	{
		if (Impl.IsValid())
		{
			return Impl->GetSampleRate();
		}
		return 0;
	}

	bool FAudioCapture::IsStreamOpen() const
	{
		if (Impl.IsValid())
		{
			return Impl->IsStreamOpen();
		}
		return false;
	}

	bool FAudioCapture::IsCapturing() const
	{
		if (Impl.IsValid())
		{
			return Impl->IsCapturing();
		}
		return false;
	}

	bool FAudioCapture::GetIfHardwareFeatureIsSupported(EHardwareInputFeature FeatureType)
	{
		if (Impl.IsValid())
		{
			return Impl->GetIfHardwareFeatureIsSupported(FeatureType);
		}
		return false;
	}

	void FAudioCapture::SetHardwareFeatureEnabled(EHardwareInputFeature FeatureType, bool bIsEnabled)
	{
		if (Impl.IsValid())
		{
			Impl->SetHardwareFeatureEnabled(FeatureType, bIsEnabled);
		}
	}

	FAudioCaptureSynth::FAudioCaptureSynth()
		: NumSamplesEnqueued(0)
		, bInitialized(false)
		, bIsCapturing(false)
	{
	}

	FAudioCaptureSynth::~FAudioCaptureSynth()
	{
	}

	bool FAudioCaptureSynth::GetDefaultCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo)
	{
		return AudioCapture.GetCaptureDeviceInfo(OutInfo);
	}

	bool FAudioCaptureSynth::OpenDefaultStream()
	{
		bool bSuccess = true;
		if (!AudioCapture.IsStreamOpen())
		{
			FOnAudioCaptureFunction OnCapture = [this](const void* AudioData, int32 NumFrames, int32 NumChannels, int32 SampleRate, double StreamTime, bool bOverFlow)
			{
				int32 NumSamples = NumChannels * NumFrames;

				FScopeLock Lock(&CaptureCriticalSection);

				if (bIsCapturing)
				{
					// Append the audio memory to the capture data buffer
					int32 Index = AudioCaptureData.AddUninitialized(NumSamples);
					float* AudioCaptureDataPtr = AudioCaptureData.GetData();

					//Avoid reading outside of buffer boundaries
					if (!(AudioCaptureData.Num() + NumSamples > MaxBufferSize))
					{
						FMemory::Memcpy(&AudioCaptureDataPtr[Index], AudioData, NumSamples * sizeof(float));
					}
					else
					{
						UE_LOGF(LogAudioCaptureCore, Warning, "Attempt to write past end of buffer in OpenDefaultStream [%u]", AudioCaptureData.Num() + NumSamples);
					}
				}
			};

			// Prepare the audio buffer memory for 2 seconds of stereo audio at 48k SR to reduce chance for allocation in callbacks
			AudioCaptureData.Reserve(2 * 2 * 48000);

			FAudioCaptureDeviceParams Params = FAudioCaptureDeviceParams();

			// Start the stream here to avoid hitching the audio render thread.
			if (AudioCapture.OpenAudioCaptureStream(Params, MoveTemp(OnCapture), 1024))
			{
				if (!AudioCapture.StartStream())
				{
					UE_LOGF(LogAudioCaptureCore, Error, "FAudioCaptureSynth::OpenDefaultStream: Failed to start audio capture stream.");
					AudioCapture.CloseStream();
					bSuccess = false;
				}
			}
			else
			{
				bSuccess = false;
			}
		}
		return bSuccess;
	}

	bool FAudioCaptureSynth::StartCapturing()
	{
		FScopeLock Lock(&CaptureCriticalSection);

		AudioCaptureData.Reset();

		check(AudioCapture.IsStreamOpen());

		bIsCapturing = true;
		return true;
	}

	void FAudioCaptureSynth::StopCapturing()
	{
		check(AudioCapture.IsStreamOpen());
		check(AudioCapture.IsCapturing());
		FScopeLock Lock(&CaptureCriticalSection);
		bIsCapturing = false;
	}

	void FAudioCaptureSynth::AbortCapturing()
	{
		AudioCapture.AbortStream();
		AudioCapture.CloseStream();
	}

	bool FAudioCaptureSynth::IsStreamOpen() const
	{
		return AudioCapture.IsStreamOpen();
	}

	bool FAudioCaptureSynth::IsCapturing() const
	{
		return bIsCapturing;
	}

	int32 FAudioCaptureSynth::GetNumSamplesEnqueued()
	{
		FScopeLock Lock(&CaptureCriticalSection);
		return AudioCaptureData.Num();
	}

	bool FAudioCaptureSynth::GetAudioData(TArray<float>& OutAudioData)
	{
		FScopeLock Lock(&CaptureCriticalSection);

		int32 CaptureDataSamples = AudioCaptureData.Num();
		if (CaptureDataSamples > 0)
		{
			// Append the capture audio to the output buffer
			int32 OutIndex = OutAudioData.AddUninitialized(CaptureDataSamples);
			float* OutDataPtr = OutAudioData.GetData();

			//Check bounds of buffer
			if (!(OutIndex > MaxBufferSize))
			{
				FMemory::Memcpy(&OutDataPtr[OutIndex], AudioCaptureData.GetData(), CaptureDataSamples * sizeof(float));
			}
			else
			{
				UE_LOGF(LogAudioCaptureCore, Warning, "Attempt to write past end of buffer in GetAudioData");
				return false;
			}

			// Reset the capture data buffer since we copied the audio out
			AudioCaptureData.Reset();
			return true;
		}
		return false;
	}

} // namespace audio
