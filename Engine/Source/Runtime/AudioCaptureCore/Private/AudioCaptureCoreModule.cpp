// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureCoreModule.h"
#include "AudioCaptureCoreLog.h"

#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAudioCaptureCore)


void FAudioCaptureCoreModule::StartupModule()
{

}

void FAudioCaptureCoreModule::ShutdownModule()
{

}


#if defined(REQUIRE_EXPLICIT_GMALLOC_INIT) && REQUIRE_EXPLICIT_GMALLOC_INIT
extern "C" void IMPLEMENT_MODULE_AudioCaptureCore() { }
#else
IMPLEMENT_MODULE(FAudioCaptureCoreModule, AudioCaptureCore);
#endif
