// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureCore.h"
#include "AudioCaptureCoreLog.h"
#include "Misc/App.h"
#include "Features/IModularFeatures.h"

namespace Audio
{
	// Null implementation for compiler
	class FNullAudioCaptureDevice : public IAudioCaptureStream
	{
	public:
		FNullAudioCaptureDevice() {}

		// Begin IAudioCaptureStream
		virtual bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex) override { return false; }
		virtual bool OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction OnCapture, uint32 NumFramesDesired) override { return false; }
		virtual bool CloseStream() override { return false; }
		virtual bool StartStream() override { return false; }
		virtual bool StopStream() override { return false; }
		virtual bool AbortStream() override { return false; }
		virtual bool GetStreamTime(double& OutStreamTime) override { return false; }
		virtual int32 GetSampleRate() const override { return 0; }
		virtual bool IsStreamOpen() const override { return false; }
		virtual bool IsCapturing() const override { return false; }
		virtual void OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow) override {}
		virtual bool GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices) override { return false; }
		// ~End IAudioCaptureStream
	};

	FORCEINLINE TUniquePtr<IAudioCaptureStream> FAudioCapture::CreateImpl()
	{
		IModularFeatures::Get().LockModularFeatureList();
		TArray<IAudioCaptureFactory*> AudioCaptureStreamFactories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioCaptureFactory>(IAudioCaptureFactory::GetModularFeatureName());
		IModularFeatures::Get().UnlockModularFeatureList();

		// For now, just return the first audio capture stream implemented. We can make this configurable at a later point.
		if (AudioCaptureStreamFactories.Num() > 0 && AudioCaptureStreamFactories[0] != nullptr && FApp::CanEverRenderAudio())
		{
			return AudioCaptureStreamFactories[0]->CreateNewAudioCaptureStream();
		}
		else
		{
			UE_LOGF(LogAudioCaptureCore, Display, "No Audio Capture implementations found. Audio input will be silent.");
			return TUniquePtr<IAudioCaptureStream>(new FNullAudioCaptureDevice());
		}
	}

} // namespace audio
