// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioCaptureDeviceInterface.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "DSP/Delay.h"
#include "DSP/EnvelopeFollower.h"
#include "HAL/CriticalSection.h"
#include "HAL/ThreadSafeBool.h"
#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

#define UE_API AUDIOCAPTURECORE_API

namespace Audio
{
	// Various hardware-accelerated features that an input device can have.
	enum class EHardwareInputFeature : uint8
	{
		EchoCancellation,
		NoiseSuppression,
		AutomaticGainControl
	};

	// Class which handles audio capture internally, implemented with a back-end per platform
	class FAudioCapture
	{
	public:

		UE_API FAudioCapture();
		UE_API ~FAudioCapture();

		// Returns the total amount of audio devices.
		UE_API int32 GetCaptureDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices);

		// Adds a user to the system so we can use its devices.
		UE_API bool RegisterUser(const TCHAR* UserId);

		// Removes a user added with RegisterUser.
		UE_API bool UnregisterUser(const TCHAR* UserId);

		// Returns the audio capture device information at the given Id.
		UE_API bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex = INDEX_NONE);

		// Opens the audio capture stream with the given parameters
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.3, "OpenCaptureStream is deprecated, please use OpenAudioCaptureStream instead.")
		UE_API bool OpenCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnCaptureFunction OnCapture, uint32 NumFramesDesired);
		UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Opens the audio capture stream with the given parameters
		bool OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction OnCapture, uint32 NumFramesDesired);

		// Closes the audio capture stream
		UE_API bool CloseStream();

		// Start the audio capture stream
		UE_API bool StartStream();

		// Stop the audio capture stream
		UE_API bool StopStream();

		// Abort the audio capture stream (stop and close)
		UE_API bool AbortStream();

		// Get the stream time of the audio capture stream
		UE_API bool GetStreamTime(double& OutStreamTime) const;

		// Get the sample rate in use by the stream.
		UE_API int32 GetSampleRate() const;

		// Returns if the audio capture stream has been opened.
		UE_API bool IsStreamOpen() const;

		// Returns true if the audio capture stream is currently capturing audio
		UE_API bool IsCapturing() const;

		UE_API bool GetIfHardwareFeatureIsSupported(EHardwareInputFeature FeatureType);

		UE_API void SetHardwareFeatureEnabled(EHardwareInputFeature FeatureType, bool bIsEnabled);

	private:

		TUniquePtr<IAudioCaptureStream> CreateImpl();
		TUniquePtr<IAudioCaptureStream> Impl;
	};

	/** Class which contains an FAudioCapture object and performs analysis on the audio stream, only outputing audio if it matches a detection criteria. */
	class FAudioCaptureSynth
	{
	public:
		UE_API FAudioCaptureSynth();
		UE_API virtual ~FAudioCaptureSynth();

		// Gets the default capture device info
		UE_API bool GetDefaultCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo);

		// Opens up a stream to the default capture device
		UE_API bool OpenDefaultStream();

		// Starts capturing audio
		UE_API bool StartCapturing();

		// Stops capturing audio
		UE_API void StopCapturing();

		// Immediately stop capturing audio
		UE_API void AbortCapturing();

		// Returned if the capture synth is closed
		UE_API bool IsStreamOpen() const;

		// Returns true if the capture synth is capturing audio
		UE_API bool IsCapturing() const;

		// Retrieves audio data from the capture synth.
		// This returns audio only if there was non-zero audio since this function was last called.
		UE_API bool GetAudioData(TArray<float>& OutAudioData);

		// Returns the number of samples enqueued in the capture synth
		UE_API int32 GetNumSamplesEnqueued();

	private:

		// Number of samples enqueued
		int32 NumSamplesEnqueued;

		// Information about the default capture device we're going to use
		FCaptureDeviceInfo CaptureInfo;

		// Audio capture object dealing with getting audio callbacks
		FAudioCapture AudioCapture;

		// Critical section to prevent reading and writing from the captured buffer at the same time
		FCriticalSection CaptureCriticalSection;

		// Buffer of audio capture data, yet to be copied to the output 
		TArray<float> AudioCaptureData;


		// If the object has been initialized
		bool bInitialized;

		// If we're capturing data
		bool bIsCapturing;
	};

} // namespace Audio

#undef UE_API
