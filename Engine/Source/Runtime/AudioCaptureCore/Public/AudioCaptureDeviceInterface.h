// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"


namespace Audio
{
	enum class EHardwareInputFeature : uint8;
	
	/**
	 * PCM audio format (bits per sample) supported by device layers (WASAPI, CoreAudio, etc.)
	 */
	enum class EPCMAudioEncoding : uint8
	{
		UNKNOWN,
		PCM_8,
		PCM_16,
		PCM_24,
		PCM_24_IN_32,
		PCM_32,
		FLOATING_POINT_32,
		FLOATING_POINT_64
	};

	struct FCaptureDeviceInfo
 	{
		FString DeviceName;
		FString DeviceId;
		int32 InputChannels;
		int32 PreferredSampleRate;
		bool bSupportsHardwareAEC;
	};

	// Indicates the default, system capture device should be used for capturing audio
	constexpr int32 DefaultDeviceIndex = INDEX_NONE;
	// Indicates invalid sample rate and that the default device's sample rate should be used
	constexpr int32 InvalidDeviceSampleRate = -1;
	// Indicates invalid channel count and that the input channel count of the default device should be used
	constexpr int32 InvalidDeviceChannelCount = -1;
	// Some clients such as UAudioCaptureComponent assume 32-bit float bit depth
	constexpr EPCMAudioEncoding DefaultDeviceEncoding = EPCMAudioEncoding::FLOATING_POINT_32;

	struct FAudioCaptureDeviceParams
	{
		// Set to true to use this device's built-in echo cancellation, if possible.
		bool bUseHardwareAEC = false;
		// Set this to INDEX_NONE 
		int32 DeviceIndex = DefaultDeviceIndex;
		// Number of input channels on the device available for recording audio
		int32 NumInputChannels = InvalidDeviceChannelCount;
		// The sample rate to use when recording audio on this device
		int32 SampleRate = InvalidDeviceSampleRate;
		// The bit depth to use when recording audio on this device
		EPCMAudioEncoding PCMAudioEncoding = DefaultDeviceEncoding;
	};

	UE_DEPRECATED(5.3, "FOnCaptureFunction is deprecated, please use FOnAudioCaptureFunction instead.")
	typedef TFunction<void(const float* InAudio, int32 NumFrames, int32 NumChannels, int32 SampleRate, double StreamTime, bool bOverFlow)> FOnCaptureFunction;
	// Callable function type used for audio capture callbacks.
	typedef TFunction<void(const void* InAudio, int32 NumFrames, int32 NumChannels, int32 SampleRate, double StreamTime, bool bOverFlow)> FOnAudioCaptureFunction;

	class IAudioCaptureStream : public IModularFeature
	{
	public:
		IAudioCaptureStream() {}
		virtual ~IAudioCaptureStream() = default;

		
		// Lets us know which users are in the system.
		virtual bool RegisterUser(const TCHAR* UserId) { return true; }
		// Call this to remove a user that was added with RegisterUser.
		virtual bool UnregisterUser(const TCHAR* UserId) { return true; }
		// Returns the audio capture device information at the given Id.
		virtual bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex) = 0;
		// Opens the audio capture stream with the given parameters
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.3, "OpenCaptureStream is deprecated, please use OpenAudioCaptureStream instead.")
		virtual bool OpenCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnCaptureFunction InOnCapture, uint32 NumFramesDesired)
		{
			FOnAudioCaptureFunction OnCaptureCallback = [InOnCapture](const void* InBuffer, int32 InNumFrames, int32 InNumChannels, int32 InSampleRate, double InStreamTime, bool bInOverflow)
			{
				InOnCapture((const float*)InBuffer, InNumFrames, InNumChannels, InSampleRate, InStreamTime, bInOverflow);
			};

			return OpenAudioCaptureStream(InParams, OnCaptureCallback, NumFramesDesired);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		// Opens the audio capture stream with the given parameters and using FOnAudioCaptureFunction for callbacks
		virtual bool OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired) = 0;
		// Closes the audio capture stream
		virtual bool CloseStream() = 0;
		// Start the audio capture stream
		virtual bool StartStream() = 0;
		// Stop the audio capture stream
		virtual bool StopStream() = 0;
		// Abort the audio capture stream (stop and close)
		virtual bool AbortStream() = 0;
		// Get the stream time of the audio capture stream
		virtual bool GetStreamTime(double& OutStreamTime) = 0;
		// Get the sample rate in use by the stream.
		virtual int32 GetSampleRate() const = 0;
		// Returns if the audio capture stream has been opened.
		virtual bool IsStreamOpen() const = 0;
		// Returns true if the audio capture stream is currently capturing audio.
		virtual bool IsCapturing() const = 0;

		// This is the callback for querying audio from the input device.
		virtual void OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow) = 0;

		// Returns the total amount of audio devices.
		virtual bool GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices) = 0;

		virtual bool GetIfHardwareFeatureIsSupported(EHardwareInputFeature FeatureType) 
		{ 
			return false;
		}

		virtual void SetHardwareFeatureEnabled(EHardwareInputFeature FeatureType, bool bEnabled)
		{
		}
	};

	class IAudioCaptureFactory : public IModularFeature
	{
	public:
		IAudioCaptureFactory() {}
		virtual ~IAudioCaptureFactory() = default;

		// IModularFeature name. Use this to register an implementation of IAudioCaptureDevice.
		static FName GetModularFeatureName()
		{
			static FName AudioCaptureFeatureName = FName("AudioCaptureStream");
			return AudioCaptureFeatureName;
		}

		virtual TUniquePtr<IAudioCaptureStream> CreateNewAudioCaptureStream() = 0;
	};

} // namespace Audio
